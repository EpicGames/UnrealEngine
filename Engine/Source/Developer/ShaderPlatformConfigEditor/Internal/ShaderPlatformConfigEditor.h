// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FName;
class IDetailLayoutBuilder;
class ITargetPlatformSettings;

enum EShaderPlatform : uint16;

namespace UE::ShaderPlatformConfigEditor
{
	SHADERPLATFORMCONFIGEDITOR_API void AddShaderPlatformConfig(IDetailLayoutBuilder& DetailBuilder, EShaderPlatform ShaderPlatform);
	SHADERPLATFORMCONFIGEDITOR_API void AddShaderPlatformConfig(IDetailLayoutBuilder& DetailBuilder, FName ShaderFormat);

	// Shortcuts for adding all shader formats/platforms based on ITargetPlatformSettings
	SHADERPLATFORMCONFIGEDITOR_API void AddShaderPlatformConfigs(IDetailLayoutBuilder& DetailBuilder, const ITargetPlatformSettings* TargetPlatformSettings);
	SHADERPLATFORMCONFIGEDITOR_API void AddShaderPlatformConfigsFromSettingsModule(IDetailLayoutBuilder& DetailBuilder, FName SettingsModuleName);
}
