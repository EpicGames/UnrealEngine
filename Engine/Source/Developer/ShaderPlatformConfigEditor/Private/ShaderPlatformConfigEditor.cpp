// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderPlatformConfigEditor.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"
#include "RHIShaderPlatformConfig.h"
#include "RHIStrings.h"
#include "ShaderPlatformConfigSettings.h"

#define LOCTEXT_NAMESPACE "ShaderPlatformConfig"

static UShaderPlatformConfigSettings* GetOrCreateShaderPlatformConfigSettings(EShaderPlatform InShaderPlatform)
{
	TStringBuilder<64> ConfigSection;
	UE::RHIShaderPlatformConfig::GetConfigSectionNameForShaderPlatform(ConfigSection, InShaderPlatform);

	FString SettingsName(ConfigSection);
	SettingsName.ReplaceInline(TEXT(" "), TEXT("_"));

	UClass* SettingsClass = UShaderPlatformConfigSettings::StaticClass();
	if (UShaderPlatformConfigSettings* ExistingSettings = FindObject<UShaderPlatformConfigSettings>(SettingsClass, SettingsName))
	{
		return ExistingSettings;
	}

	UShaderPlatformConfigSettings* NewSettings = NewObject<UShaderPlatformConfigSettings>(SettingsClass, FName(SettingsName));

	NewSettings->SetConfiguration(InShaderPlatform, ConfigSection.ToString());

	return NewSettings;
}

void UE::ShaderPlatformConfigEditor::AddShaderPlatformConfig(IDetailLayoutBuilder& DetailBuilder, EShaderPlatform InShaderPlatform)
{
	// ue-todo: figure out how to filter out platforms if they don't have any editable properties.
	if (!FDataDrivenShaderPlatformInfo::GetSupportsBindless(InShaderPlatform))
	{
		return;
	}

	const FText ShaderPlatformFriendlyName = FDataDrivenShaderPlatformInfo::GetFriendlyName(InShaderPlatform);

	const FText CategoryText = FText::Format(NSLOCTEXT("ShaderPlatformConfig", "CategoryFormat", "Shader Platform Config: {0}"), ShaderPlatformFriendlyName);

	UShaderPlatformConfigSettings* ShaderPlatformConfigSettings = GetOrCreateShaderPlatformConfigSettings(InShaderPlatform);

	IDetailCategoryBuilder& NewCategory = DetailBuilder.EditCategory(ShaderPlatformConfigSettings->GetFName(), CategoryText, ECategoryPriority::Important);
	FAddPropertyParams Params = FAddPropertyParams()
		.ForceShowProperty()
		.HideRootObjectNode(true)
		.CreateCategoryNodes(false);
	NewCategory.AddExternalObjects({ ShaderPlatformConfigSettings }, EPropertyLocation::Default, Params);
}

void UE::ShaderPlatformConfigEditor::AddShaderPlatformConfig(IDetailLayoutBuilder& DetailBuilder, FName ShaderFormat)
{
	const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);
	if (ShaderPlatform != SP_NumPlatforms)
	{
		UE::ShaderPlatformConfigEditor::AddShaderPlatformConfig(DetailBuilder, ShaderPlatform);
	}
}

void UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigs(IDetailLayoutBuilder& DetailBuilder, const ITargetPlatformSettings* TargetPlatformSettings)
{
	if (TargetPlatformSettings)
	{
		TArray<FName> ShaderFormats;
		TargetPlatformSettings->GetAllPossibleShaderFormats(ShaderFormats);

		for (const FName ShaderFormat : ShaderFormats)
		{
			UE::ShaderPlatformConfigEditor::AddShaderPlatformConfig(DetailBuilder, ShaderFormat);
		}
	}
}

void UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigsFromSettingsModule(IDetailLayoutBuilder& DetailBuilder, FName SettingsModuleName)
{
	if (ITargetPlatformSettingsModule* Module = FModuleManager::GetModulePtr<ITargetPlatformSettingsModule>(SettingsModuleName))
	{
		const TArray<ITargetPlatformSettings*> AllSettings = Module->GetTargetPlatformSettings();
		for (const ITargetPlatformSettings* TargetPlatformSettings : AllSettings)
		{
			UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigs(DetailBuilder, TargetPlatformSettings);
		}
	}
}

#undef LOCTEXT_NAMESPACE
