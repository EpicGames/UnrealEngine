// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderPlatformConfigSettings.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "RHI.h"
#include "RHIStrings.h"
#include "ShaderPlatformConfig.h"

inline ERHIBindlessConfiguration RHIBindlessConfigurationFromSetting(EBindlessConfigurationSetting InSetting)
{
	switch (InSetting)
	{
	default:
	case EBindlessConfigurationSetting::Disabled:   return ERHIBindlessConfiguration::Disabled;
	case EBindlessConfigurationSetting::RayTracing: return ERHIBindlessConfiguration::RayTracing;
	case EBindlessConfigurationSetting::Minimal:    return ERHIBindlessConfiguration::Minimal;
	case EBindlessConfigurationSetting::All:        return ERHIBindlessConfiguration::All;
	}
}

UShaderPlatformConfigSettings::UShaderPlatformConfigSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const TCHAR* UShaderPlatformConfigSettings::GetConfigOverridePlatform() const
{
	return *ConfigPlatform;
}

void UShaderPlatformConfigSettings::OverridePerObjectConfigSection(FString& InOutSectionName)
{
	if (!ConfigSection.IsEmpty())
	{
		InOutSectionName = ConfigSection;
	}
}

bool UShaderPlatformConfigSettings::CanEditBindlessConfiguration() const
{
	// UE-TODO - Carl: Don't expose Metal SM6 platforms here as they must use bindless, we need a more generic way to handle this
	return FDataDrivenShaderPlatformInfo::GetSupportsBindless(ShaderPlatform) && !IsMetalSM6Platform(ShaderPlatform);
}

bool UShaderPlatformConfigSettings::IsBindlessEnabledForGraphics() const
{
	return IsBindlessEnabledForAnyGraphics(RHIBindlessConfigurationFromSetting(BindlessConfiguration));
}

void UShaderPlatformConfigSettings::SetConfiguration(EShaderPlatform InShaderPlatform, const TCHAR* InConfigSection)
{
	ShaderPlatform = InShaderPlatform;
	ConfigPlatform = ShaderPlatformToPlatformName(InShaderPlatform).ToString();
	ConfigSection = InConfigSection;

	LoadConfig();
}
