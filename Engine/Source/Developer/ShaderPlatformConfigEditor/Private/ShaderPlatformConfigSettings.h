// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIShaderPlatform.h"
#include "UObject/ObjectMacros.h"

#include "ShaderPlatformConfigSettings.generated.h"

UENUM()
enum class EBindlessConfigurationSetting : uint8
{
	Disabled    UMETA(DisplayName = "Disabled",    ToolTip = "Bindless rendering is disabled. All resources use traditional binding."),
	RayTracing  UMETA(DisplayName = "Ray Tracing", ToolTip = "Bindless enabled for Ray Tracing shaders only."),
	Minimal     UMETA(DisplayName = "Minimal",     ToolTip = "Bindless enabled for a minimal set of shaders, notably Ray Tracing shaders and surface Materials."),
	All         UMETA(DisplayName = "All",         ToolTip = "Bindless enabled for all shaders - may reduce performance (Experimental)."),
};

// Temporary solution for getting editing up and running, should be replaced with custom Slate code in the future
UCLASS(MinimalAPI, config=Engine, DefaultConfig, PerObjectConfig)
class UShaderPlatformConfigSettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	UPROPERTY(config, EditAnywhere, Category="ShaderPlatformConfigSettings",
		meta = (EditCondition = "CanEditBindlessConfiguration()", ToolTip = "Set Bindless configuration for this shader platform."))
	EBindlessConfigurationSetting BindlessConfiguration = EBindlessConfigurationSetting::Disabled;

	UPROPERTY(config, EditAnywhere, Category="ShaderPlatformConfigSettings",
		meta = (EditCondition = "IsBindlessEnabledForGraphics()", ToolTip = "Enable Nanite bindless-aware shading for this shader platform. Only usable when Bindless is Minimal or higher. (Experimental)"))
	bool bEnableNaniteBindlessShading = false;

	UPROPERTY(config, EditAnywhere, Category="ShaderPlatformConfigSettings",
		meta = (EditCondition = "IsBindlessEnabledForGraphics()", ToolTip = "Enable Nanite bindless-aware rasterization for this shader platform. Only usable when Bindless is Minimal or higher. (Experimental)"))
	bool bEnableNaniteBindlessRasterization = false;

	virtual const TCHAR* GetConfigOverridePlatform() const final;
	virtual void OverridePerObjectConfigSection(FString& InOutSectionName) final;

	UFUNCTION()
	bool CanEditBindlessConfiguration() const;

	UFUNCTION()
	bool IsBindlessEnabledForGraphics() const;

	void SetConfiguration(EShaderPlatform InShaderPlatform, const TCHAR* InConfigSection);

private:
	EShaderPlatform ShaderPlatform{};
	FString ConfigPlatform;
	FString ConfigSection;
};
