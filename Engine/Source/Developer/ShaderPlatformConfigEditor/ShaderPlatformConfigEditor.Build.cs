// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ShaderPlatformConfigEditor : ModuleRules
{
	public ShaderPlatformConfigEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
				"RHI",
				"RenderCore",

				"PropertyEditor",
				"UnrealEd",
				"CoreUObject",
				"TargetPlatform",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Engine",
				"RHI",
			}
		);
	}
}
